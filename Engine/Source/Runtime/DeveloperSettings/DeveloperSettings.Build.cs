// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DeveloperSettings : ModuleRules
{
	public DeveloperSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Projects"
			}
		);
	}
}
