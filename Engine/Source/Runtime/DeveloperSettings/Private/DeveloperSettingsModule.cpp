// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class FDeveloperSettingsModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

void FDeveloperSettingsModule::StartupModule()
{
}

void FDeveloperSettingsModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FDeveloperSettingsModule, DeveloperSettings );
