// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DeveloperSettings.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeveloperSettings)

UDeveloperSettings::UDeveloperSettings(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
	CategoryName = NAME_None;
	SectionName = NAME_None;
}

FName UDeveloperSettings::GetContainerName() const
{
	static const FName ProjectName("Project");
	static const FName EditorName("Editor");

	static const FName EditorSettingsName("EditorSettings");
	static const FName EditorPerProjectUserSettingsName("EditorPerProjectUserSettings");

	FName ConfigName = GetClass()->ClassConfigName;

	if ( ConfigName == EditorSettingsName || ConfigName == EditorPerProjectUserSettingsName )
	{
		return EditorName;
	}
	
	return ProjectName;
}

FName UDeveloperSettings::GetCategoryName() const
{
	static const FName GeneralName("General");
	static const FName EditorSettingsName("EditorSettings");
	static const FName EditorPerProjectUserSettingsName("EditorPerProjectUserSettings");

	if ( CategoryName != NAME_None )
	{
		return CategoryName;
	}

	FName ConfigName = GetClass()->ClassConfigName;
	if ( ConfigName == NAME_Engine || ConfigName == NAME_Input )
	{
		return NAME_Engine;
	}
	else if ( ConfigName == EditorSettingsName || ConfigName == EditorPerProjectUserSettingsName )
	{
		return GeneralName;
	}
	else if ( ConfigName == NAME_Editor || ConfigName == NAME_EditorSettings || ConfigName == NAME_EditorLayout || ConfigName == NAME_EditorKeyBindings )
	{
		return NAME_Editor;
	}
	else if ( ConfigName == NAME_Game )
	{
		return NAME_Game;
	}

	return NAME_Engine;
}

FName UDeveloperSettings::GetSectionName() const
{
	if ( SectionName != NAME_None )
	{
		return SectionName;
	}

	return GetClass()->GetFName();
}

#if WITH_EDITOR
FText UDeveloperSettings::GetSectionText() const
{
	return GetClass()->GetDisplayNameText();
}

FText UDeveloperSettings::GetSectionDescription() const
{
	return GetClass()->GetToolTipText();
}

void UDeveloperSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	SettingsChangedDelegate.Broadcast(this, PropertyChangedEvent);
}

UDeveloperSettings::FOnSettingsChanged& UDeveloperSettings::OnSettingChanged()
{
	return SettingsChangedDelegate;
}
#endif

TSharedPtr<SWidget> UDeveloperSettings::GetCustomSettingsWidget() const
{
	return TSharedPtr<SWidget>();
}


#if WITH_EDITOR

static FName DeveloperSettingsConsoleVariableMetaFName("ConsoleVariable");

void UDeveloperSettings::ImportConsoleVariableValues()
{
	for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		const FString& CVarName = Property->GetMetaData(DeveloperSettingsConsoleVariableMetaFName);
		if (!CVarName.IsEmpty())
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
			if (CVar)
			{
				if (Property->ImportText_InContainer(*CVar->GetString(), this, this, PPF_ConsoleVariable) == NULL)
				{
					UE_LOGF(LogInit, Error, "%ls import failed for %ls on console variable %ls (=%ls)", *GetClass()->GetName(), *Property->GetName(), *CVarName, *CVar->GetString());
				}
			}
			else
			{
				UE_LOGF(LogInit, Error, "%ls failed to find console variable %ls for %ls", *GetClass()->GetName(), *CVarName, *Property->GetName());
			}
		}
	}
}

void UDeveloperSettings::ExportValuesToConsoleVariables(FProperty* PropertyThatChanged)
{
	if(PropertyThatChanged)
	{
		const FString& CVarName = PropertyThatChanged->GetMetaData(DeveloperSettingsConsoleVariableMetaFName);
		if (!CVarName.IsEmpty())
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
			if (CVar && (CVar->GetFlags() & ECVF_ReadOnly) == 0)
			{
				FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyThatChanged);
				if (ByteProperty != NULL && ByteProperty->Enum != NULL)
				{
					CVar->Set(ByteProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
				}
				else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyThatChanged))
				{
					FNumericProperty* UnderlyingProp = EnumProperty->GetUnderlyingProperty();
					void* PropertyAddress = EnumProperty->ContainerPtrToValuePtr<void>(this);
					CVar->Set((int32)UnderlyingProp->GetSignedIntPropertyValue(PropertyAddress), ECVF_SetByProjectSetting);
				}
				else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(PropertyThatChanged))
				{
					CVar->Set((int32)BoolProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
				}
				else if (FIntProperty* IntProperty = CastField<FIntProperty>(PropertyThatChanged))
				{
					CVar->Set(IntProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
				}
				else if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(PropertyThatChanged))
				{
					CVar->Set(FloatProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
				}
				else if (FStrProperty* StringProperty = CastField<FStrProperty>(PropertyThatChanged))
				{
					CVar->Set(*StringProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
				}
				else if (FNameProperty* NameProperty = CastField<FNameProperty>(PropertyThatChanged))
				{
					CVar->Set(*NameProperty->GetPropertyValue_InContainer(this).ToString(), ECVF_SetByProjectSetting);
				}

			}
			else
			{
				// Reduce the amount of log spam for read-only properties. 
				// We assume that if property requires restart it is very likely it needs to stay read-only and therefore no need to log a warning.
				static const FName ConfigRestartRequiredKey = "ConfigRestartRequired";
				if (!PropertyThatChanged->GetBoolMetaData(ConfigRestartRequiredKey))
				{
					UE_LOGF(LogInit, Warning, "CVar named '%ls' marked up in %ls was not found or is set to read-only", *CVarName, *GetClass()->GetName());
				}
			}
		}
	}
}

#endif

