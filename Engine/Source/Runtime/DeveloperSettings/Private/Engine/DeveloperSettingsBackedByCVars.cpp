// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/DeveloperSettingsBackedByCVars.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeveloperSettingsBackedByCVars)

UDeveloperSettingsBackedByCVars::UDeveloperSettingsBackedByCVars(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDeveloperSettingsBackedByCVars::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif
}

#if WITH_EDITOR
void UDeveloperSettingsBackedByCVars::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif

