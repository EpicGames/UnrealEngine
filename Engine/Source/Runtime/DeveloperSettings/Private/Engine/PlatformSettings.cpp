// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PlatformSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformSettings)

void FPerPlatformSettings::Initialize(TSubclassOf<UPlatformSettings> SettingsClass)
{
#if WITH_EDITOR
	Settings = UPlatformSettingsManager::Get().GetAllPlatformSettings(SettingsClass);
#endif
}

UPlatformSettings::UPlatformSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

