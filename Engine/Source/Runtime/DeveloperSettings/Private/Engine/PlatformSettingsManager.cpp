// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PlatformSettingsManager.h"
#include "Engine/PlatformSettings.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "HAL/PlatformProperties.h"
#include "Interfaces/IProjectManager.h"
#include "ProjectDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlatformSettingsManager)

#if WITH_EDITOR
FName UPlatformSettingsManager::SimulatedEditorPlatform;
#endif

UPlatformSettingsManager::UPlatformSettingsManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, IniPlatformName(FPlatformProperties::IniPlatformName())
{
}

UPlatformSettings* UPlatformSettingsManager::GetSettingsForPlatform(TSubclassOf<UPlatformSettings> SettingsClass) const
{
#if WITH_EDITOR
	if (GIsEditor && !SimulatedEditorPlatform.IsNone())
	{
		return GetSettingsForPlatform(SettingsClass, SimulatedEditorPlatform);
	}
#endif

	return GetSettingsForPlatform(SettingsClass, IniPlatformName);
}

UPlatformSettings* UPlatformSettingsManager::GetSettingsForPlatform(TSubclassOf<UPlatformSettings> SettingsClass, FName DesiredPlatformIniName) const
{
	check(SettingsClass != nullptr);

	FPlatformSettingsInstances& Record = SettingsMap.FindOrAdd(SettingsClass);

	TObjectPtr<UPlatformSettings>& Result = (DesiredPlatformIniName == IniPlatformName) ? Record.PlatformInstance : Record.OtherPlatforms.FindOrAdd(DesiredPlatformIniName);
	if (Result == nullptr)
	{
		Result = CreateSettingsObjectForPlatform(SettingsClass, DesiredPlatformIniName);
	}
	return Result;
}

FName UPlatformSettingsManager::GetCurrentPlatformName() const
{
#if WITH_EDITOR
	if (GIsEditor && !SimulatedEditorPlatform.IsNone())
	{
		return SimulatedEditorPlatform;
	}
#endif
	return IniPlatformName;
}

UPlatformSettings* UPlatformSettingsManager::CreateSettingsObjectForPlatform(TSubclassOf<UPlatformSettings> SettingsClass, FName TargetIniPlatformName) const
{
	const FString PlatformSettingsName = SettingsClass->GetName() + TEXT("_") + TargetIniPlatformName.ToString();
	UPlatformSettings* PlatformSettingsForThisClass = NewObject<UPlatformSettings>(GetTransientPackage(), SettingsClass, FName(*PlatformSettingsName));
	PlatformSettingsForThisClass->AddToRoot();
	PlatformSettingsForThisClass->ConfigPlatformName = TargetIniPlatformName;
	PlatformSettingsForThisClass->ConfigPlatformNameStr = TargetIniPlatformName.ToString();

	const TMap<FName, FDataDrivenPlatformInfo>& AllPlatforms = FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos();
	if (ensureMsgf(AllPlatforms.Contains(TargetIniPlatformName), TEXT("Platform config %s requested for unknown platform %s"), *SettingsClass->GetPathName(), *TargetIniPlatformName.ToString()))
	{
		PlatformSettingsForThisClass->InitializePlatformDefaults();

#if WITH_EDITOR
		PlatformSettingsForThisClass->LoadConfig();
#else
		//Note: Calling PostLoad will implicitly call LoadConfig. Calling just LoadConfig does not call PostLoad.
		PlatformSettingsForThisClass->PostLoad();
#endif		
	}
	
	return PlatformSettingsForThisClass;
}

#if WITH_EDITOR
TArray<FName> UPlatformSettingsManager::GetKnownAndEnablePlatformIniNames()
{
	TArray<FName> Results;

	FProjectStatus ProjectStatus;
	const bool bProjectStatusIsValid = IProjectManager::Get().QueryStatusForCurrentProject(/*out*/ ProjectStatus);

	for (const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
	{
		const FName PlatformName = Pair.Key;
		const FDataDrivenPlatformInfo& Info = Pair.Value;

		const bool bProjectDisabledPlatform = bProjectStatusIsValid && !ProjectStatus.IsTargetPlatformSupported(PlatformName);

		const bool bEnabledForUse =
#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
			Info.bEnabledForUse;
#else
			true;
#endif

		const bool bSupportedPlatform = !Info.bIsFakePlatform && bEnabledForUse && !bProjectDisabledPlatform;

		if (bSupportedPlatform)
		{
			Results.Add(PlatformName);
		}
	}

	return Results;
}

TArray<TObjectPtr<UPlatformSettings>> UPlatformSettingsManager::GetAllPlatformSettings(TSubclassOf<UPlatformSettings> SettingsClass) const
{
	TArray<TObjectPtr<UPlatformSettings>> Settings;
	for (FName PlatformIniName : GetKnownAndEnablePlatformIniNames())
	{
		Settings.Add(GetSettingsForPlatform(SettingsClass, PlatformIniName));
	}

	return Settings;
}
#endif

