// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DeveloperSettings.generated.h"

class FProperty;
class SWidget;

/**
 * The base class of any auto discovered settings object.
 */
UCLASS(Abstract, MinimalAPI)
class UDeveloperSettings : public UObject
{
	GENERATED_BODY()

public:
	DEVELOPERSETTINGS_API UDeveloperSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Gets the settings container name for the settings, either Project or Editor */
	DEVELOPERSETTINGS_API virtual FName GetContainerName() const;
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	DEVELOPERSETTINGS_API virtual FName GetCategoryName() const;
	/** The unique name for your section of settings, uses the class's FName. */
	DEVELOPERSETTINGS_API virtual FName GetSectionName() const;

#if WITH_EDITOR
	/** Gets the section text, uses the classes DisplayName by default. */
	DEVELOPERSETTINGS_API virtual FText GetSectionText() const;
	/** Gets the description for the section, uses the classes ToolTip by default. */
	DEVELOPERSETTINGS_API virtual FText GetSectionDescription() const;

	/** Whether or not this class supports auto registration or if the settings have a custom setup */
	virtual bool SupportsAutoRegistration() const { return true; }

	/** Returns a delegate that can be used to monitor for property changes to this object */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSettingsChanged, UObject*, struct FPropertyChangedEvent&);
	DEVELOPERSETTINGS_API FOnSettingsChanged& OnSettingChanged();

	/** UObject interface */
	DEVELOPERSETTINGS_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Gets a custom widget for the settings.  This is only for very custom situations. */
	DEVELOPERSETTINGS_API virtual TSharedPtr<SWidget> GetCustomSettingsWidget() const;

protected:
	/**
	 * The category name to use, overrides the one detected by looking at the config=... class metadata.
	 * Arbitrary category names are not supported, this must map to an existing category we support in the settings
	 * viewer.
	 */
	FName CategoryName;

	/** The Section name, is the short name for the settings.  If not filled in, will be the FName of the class. */
	FName SectionName;
	
#if WITH_EDITOR
	/** Populates all properties that have 'ConsoleVariable' meta data with the respective console variable values */
	DEVELOPERSETTINGS_API void ImportConsoleVariableValues();
	/** If property has 'ConsoleVariable' meta data, exports the property value to the specified console variable */
	DEVELOPERSETTINGS_API void ExportValuesToConsoleVariables(FProperty* PropertyThatChanged);

	/** Holds a delegate that is executed after the settings section has been modified. */
	FOnSettingsChanged SettingsChangedDelegate;
#endif
};
