// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DeveloperSettings.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DeveloperSettingsBackedByCVars.generated.h"

class UObject;

/**
 * The base class of auto discovered settings object where some or all of the settings
 * are stored in console variables instead of config variables.
 */
UCLASS(Abstract, MinimalAPI)
class UDeveloperSettingsBackedByCVars : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	DEVELOPERSETTINGS_API UDeveloperSettingsBackedByCVars(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	DEVELOPERSETTINGS_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	DEVELOPERSETTINGS_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

};
