// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DeveloperSettings.h"
#include "HAL/Platform.h"
#include "PlatformSettingsManager.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "PlatformSettings.generated.h"

class UPlatformSettings;
class UPlatformSettingsManager;

USTRUCT()
struct FPerPlatformSettings
{
	GENERATED_BODY()

	DEVELOPERSETTINGS_API void Initialize(TSubclassOf<UPlatformSettings> SettingsClass);

private:
	UPROPERTY(Instanced, Transient, EditAnywhere, EditFixedSize, Category = Layout)
	TArray<TObjectPtr<UPlatformSettings>> Settings;

	friend class FPerPlatformSettingsCustomization;
};

/**
 * The base class of any per platform settings.  The pattern for using these is as follows.
 * 
 * Step 1) Subclass UPlatformSettings, UMyPerPlatformSettings : public UPlatformSettings.
 * 
 * Step 2) For your system should already have a UDeveloperSettings that you created so that
 *         users can customize other properties for your system in the project.  On that class
 *         you need to create a property of type FPerPlatformSettings, e.g. 
 *         UPROPERTY(EditAnywhere, Category=Platform)
 *         FPerPlatformSettings PlatformOptions
 * 
 * Step 3) In your UDeveloperSettings subclasses construct, there should be a line like this,
 *         PlatformOptions.Settings.Initialize(UMyPerPlatformSettings::StaticClass());
 *         This will actually ensure that you initialize the settings exposed in the editor to whatever
 *         the current platform configuration is for them.
 * 
 * Step 4) Nothing else needed.  In your system code, you will just call 
 *         UMyPerPlatformSettings* MySettings = UPlatformSettingsManager::Get().GetSettingsForPlatform<UMyPerPlatformSettings>()
 *         that will get you the current settings for the active platform, or the simulated platform in the editor.
 */
UCLASS(Abstract, perObjectConfig, MinimalAPI)
class UPlatformSettings : public UObject
{
	GENERATED_BODY()

public:
	DEVELOPERSETTINGS_API UPlatformSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void InitializePlatformDefaults() { }

	FName GetPlatformIniName() const { return ConfigPlatformName; }
	
	//~UObject interface
	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return ConfigPlatformNameStr.IsEmpty() ? nullptr : *ConfigPlatformNameStr;
	}
	//~End of UObject interface

private:
	// The platform we are an instance of settings for
	FName ConfigPlatformName;
	FString ConfigPlatformNameStr;

	friend UPlatformSettingsManager;
};
