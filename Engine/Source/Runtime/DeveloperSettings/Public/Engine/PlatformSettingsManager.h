// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "DeveloperSettings.h"
#include "HAL/Platform.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "PlatformSettingsManager.generated.h"

class UPlatformSettings;

// List of platform-specific instances for a class
USTRUCT()
struct FPlatformSettingsInstances
{
	GENERATED_BODY()

	// The instance for the native platform
	UPROPERTY(Transient)
	TObjectPtr<UPlatformSettings> PlatformInstance = nullptr;

	// Instances for other platforms (only used in the editor)
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UPlatformSettings>> OtherPlatforms;
};

// The manager for all platform-specific settings
UCLASS(MinimalAPI)
class UPlatformSettingsManager : public UObject
{
	GENERATED_BODY()

public:
	DEVELOPERSETTINGS_API UPlatformSettingsManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static const UPlatformSettingsManager& Get()
	{
		return *GetDefault<UPlatformSettingsManager>();
	}

	// Returns the platform settings for the specified class for the current platform
	DEVELOPERSETTINGS_API UPlatformSettings* GetSettingsForPlatform(TSubclassOf<UPlatformSettings> SettingsClass) const;

	// Returns the platform settings for the specified class for the specified platform
	DEVELOPERSETTINGS_API UPlatformSettings* GetSettingsForPlatform(TSubclassOf<UPlatformSettings> SettingsClass, FName DesiredPlatformIniName) const;

	DEVELOPERSETTINGS_API FName GetCurrentPlatformName() const;

#if WITH_EDITOR
	// Gets the simulated platform
	static FName GetEditorSimulatedPlatform() { return SimulatedEditorPlatform; }

	// Sets the current simulated platform
	static void SetEditorSimulatedPlatform(FName PlatformIniName) { SimulatedEditorPlatform = PlatformIniName; }
#endif


public:
	template <typename TPlatformSettingsClass>
	inline TPlatformSettingsClass* GetSettingsForPlatform() const
	{
		return CastChecked<TPlatformSettingsClass>(GetSettingsForPlatform(TPlatformSettingsClass::StaticClass()));
	}

#if WITH_EDITOR
	static DEVELOPERSETTINGS_API TArray<FName> GetKnownAndEnablePlatformIniNames();

	template <typename TPlatformSettingsClass>
	inline TArray<TObjectPtr<UPlatformSettings>> GetAllPlatformSettings() const
	{
		return GetAllPlatformSettings(TPlatformSettingsClass::StaticClass());
	}

	DEVELOPERSETTINGS_API TArray<TObjectPtr<UPlatformSettings>> GetAllPlatformSettings(TSubclassOf<UPlatformSettings> SettingsClass) const;

	template <typename TPlatformSettingsClass>
	inline TPlatformSettingsClass* GetSettingsForPlatform(FName TargetIniPlatformName) const
	{
		return Cast<TPlatformSettingsClass>(GetSettingsForPlatform(TPlatformSettingsClass::StaticClass(), TargetIniPlatformName));
	}

#endif

private:
	// Creates a settings object for the specified class and platform
	DEVELOPERSETTINGS_API UPlatformSettings* CreateSettingsObjectForPlatform(TSubclassOf<UPlatformSettings> SettingsClass, FName TargetIniPlatformName) const;


private:
	// Created platform-specific settings
	UPROPERTY(Transient)
	mutable TMap<TSubclassOf<UPlatformSettings>, FPlatformSettingsInstances> SettingsMap;

	// The true platform name for the current platform (unaffected by previews, etc...)
	FName IniPlatformName;

#if WITH_EDITOR
	// Current simulated platform
	static DEVELOPERSETTINGS_API FName SimulatedEditorPlatform;
#endif
};
